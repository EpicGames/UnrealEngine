{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "LiveLinkCamera",
	"Description": "Live Link plugin adding functionalities for camera handling",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"SupportedPrograms":  ["LiveLinkHub"],
	"Plugins": [
		{
			"Name": "LiveLink",
			"Enabled": true
		},
		{
			"Name": "CameraCalibrationCore",
			"Enabled": true
		},
		{
			"Name": "LensComponent",
			"Enabled": true
		}
	],
	"Modules": [
		{
			"Name": "LiveLinkCamera",
			"Type": "Runtime",
			"ProgramAllowlist": [ "LiveLinkHub" ],
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "LiveLinkCameraEditor",
			"ProgramAllowlist": [ "LiveLinkHub" ],
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "LiveLinkCameraRecording",
			"ProgramAllowlist": [ "LiveLinkHub" ],
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		}
	]
}
