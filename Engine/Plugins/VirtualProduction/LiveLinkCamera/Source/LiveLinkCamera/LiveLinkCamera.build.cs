// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class LiveLinkCamera : ModuleRules
{
	public LiveLinkCamera(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"CameraCalibrationCore",
				"CinematicCamera",
				"Core",
				"CoreUObject",
				"Engine",
				"LiveLinkInterface",
				"LiveLink",
				"LiveLinkComponents",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"LensComponent",
			}
		);
	}
}
