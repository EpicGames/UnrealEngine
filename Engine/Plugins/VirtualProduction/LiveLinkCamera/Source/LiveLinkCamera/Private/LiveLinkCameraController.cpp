// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkCameraController.h"

#include "Camera/CameraTypes.h"
#include "CineCameraComponent.h"
#include "GameFramework/Actor.h"
#include "LensComponent.h"
#include "LiveLinkComponentController.h"
#include "LiveLinkLog.h"
#include "Roles/LiveLinkCameraRole.h"
#include "Roles/LiveLinkCameraTypes.h"
#include "UObject/EnterpriseObjectVersion.h"
#include "UObject/UE5MainStreamObjectVersion.h"

DEFINE_LOG_CATEGORY_STATIC(LogLiveLinkCameraController, Log, All);

ULiveLinkCameraController::ULiveLinkCameraController()
{
}

void ULiveLinkCameraController::Tick(float DeltaTime, const FLiveLinkSubjectFrameData& SubjectData)
{
	// Invalidate the lens file evaluation data
	LensFileEvalData.Invalidate();

	const FLiveLinkCameraStaticData* StaticData = SubjectData.StaticData.Cast<FLiveLinkCameraStaticData>();
	const FLiveLinkCameraFrameData* FrameData = SubjectData.FrameData.Cast<FLiveLinkCameraFrameData>();

	if (StaticData && FrameData)
	{
		if (StaticData->bIsFocusDistanceSupported)
		{
			LensFileEvalData.Input.Focus = FrameData->FocusDistance;
		}

		if (StaticData->bIsApertureSupported)
		{
			LensFileEvalData.Input.Iris = FrameData->Aperture;
		}

		if (StaticData->bIsFocalLengthSupported)
		{
			LensFileEvalData.Input.Zoom = FrameData->FocalLength;
		}

		if (UCameraComponent* CameraComponent = Cast<UCameraComponent>(GetAttachedComponent()))
		{
			//Stamp previous values that have an impact on frustum visual representation
			const float PreviousFOV = CameraComponent->FieldOfView;
			const float PreviousAspectRatio = CameraComponent->AspectRatio;
			const ECameraProjectionMode::Type PreviousProjectionMode = CameraComponent->ProjectionMode;

			if (StaticData->bIsFieldOfViewSupported && UpdateFlags.bApplyFieldOfView) { CameraComponent->SetFieldOfView(FrameData->FieldOfView); }
			if (StaticData->bIsAspectRatioSupported && UpdateFlags.bApplyAspectRatio) { CameraComponent->SetAspectRatio(FrameData->AspectRatio); }
			if (StaticData->bIsProjectionModeSupported && UpdateFlags.bApplyProjectionMode) { CameraComponent->SetProjectionMode(FrameData->ProjectionMode == ELiveLinkCameraProjectionMode::Perspective ? ECameraProjectionMode::Perspective : ECameraProjectionMode::Orthographic); }

			if (UCineCameraComponent* CineCameraComponent = Cast<UCineCameraComponent>(CameraComponent))
			{
				// If the depth of field support flag has changed, update the focus method accordingly
				if (bLastIsDepthOfFieldSupported != StaticData->bIsDepthOfFieldSupported)
				{
					// If depth of field is not supported, force disable depth of field on the camera
					// Otherwise, if depth of field is supported and the frame data has a valid value for focus, force the focus method to Manual 
					if (UpdateFlags.bApplyFocusDistance)
					{
						if (StaticData->bIsDepthOfFieldSupported == false)
						{
							CineCameraComponent->FocusSettings.FocusMethod = ECameraFocusMethod::Disable;
						}
						else if (StaticData->bIsFocusDistanceSupported)
						{
							CineCameraComponent->FocusSettings.FocusMethod = ECameraFocusMethod::Manual;
						}

						bLastIsDepthOfFieldSupported = StaticData->bIsDepthOfFieldSupported;
					}
				}

				ULensFile* SelectedLensFile = LensFilePicker.GetLensFile();
				LensFileEvalData.LensFile = SelectedLensFile;

				auto ConditionallyApplySensorChanges = [StaticData](float& Target, float StaticValue, float FrameValue)
				{
					if (StaticData->bIsDynamicFilmbackSupported && FrameValue > 0.0f)
					{
						Target = FrameValue;
					}
					else if (StaticValue > 0.0f)
					{
						Target = StaticValue;
					}
				};
				if (UpdateFlags.bApplyFilmBack)
				{
					ConditionallyApplySensorChanges(CineCameraComponent->Filmback.SensorWidth, StaticData->FilmBackWidth, FrameData->FilmBackWidth);
					ConditionallyApplySensorChanges(CineCameraComponent->Filmback.SensorHeight, StaticData->FilmBackHeight, FrameData->FilmBackHeight);
				}

				//Verify different Lens Tables based on streamed FIZ at some intervals
 				static const double TableVerificationInterval = 10;
 				if ((FPlatformTime::Seconds() - LastLensTableVerificationTimestamp) >= TableVerificationInterval)
 				{
 					LastLensTableVerificationTimestamp = FPlatformTime::Seconds();
 					VerifyFIZWithLensFileTables(SelectedLensFile, StaticData);
 				}

				ApplyFIZ(SelectedLensFile, CineCameraComponent, StaticData, FrameData);
			}

#if WITH_EDITORONLY_DATA

			const bool bIsVisualImpacted = (FMath::IsNearlyEqual(PreviousFOV, CameraComponent->FieldOfView) == false)
			|| (FMath::IsNearlyEqual(PreviousAspectRatio, CameraComponent->AspectRatio) == false)
			|| (PreviousProjectionMode != CameraComponent->ProjectionMode);

			if (bShouldUpdateVisualComponentOnChange && bIsVisualImpacted)
			{
				CameraComponent->RefreshVisualRepresentation();
			}
#endif
		}
	}
}

bool ULiveLinkCameraController::IsRoleSupported(const TSubclassOf<ULiveLinkRole>& RoleToSupport)
{
	return RoleToSupport == ULiveLinkCameraRole::StaticClass();
}

TSubclassOf<UActorComponent> ULiveLinkCameraController::GetDesiredComponentClass() const
{
	return UCameraComponent::StaticClass();
}

void ULiveLinkCameraController::SetApplyNodalOffset(bool bInApplyNodalOffset)
{
	// The use of this function has been deprecated.
}

void ULiveLinkCameraController::ApplyFIZ(ULensFile* LensFile, UCineCameraComponent* CineCameraComponent, const FLiveLinkCameraStaticData* StaticData, const FLiveLinkCameraFrameData* FrameData)
{
	/**
	 * The logic to apply fiz is :
	 * If there is a LensFile, give it LiveLink's Focus and Zoom to evaluate calibrated / mapped FIZ
	 * It is assumed that the LiveLink feed matches what what used to produce the lens file
	 * If no lens file is present, two choices :
	 * Use LiveLink data directly as usable FIZ. This could be coming from a tracking vendor for example
	 * Use cinecamera's min and max value range to denormalize inputs, assuming it is normalized.
	 */
	if (LensFile)
	{
		if (UpdateFlags.bApplyFocusDistance)
		{
			//If Focus encoder mapping is present, use it. If not, use incoming value directly if streamed in
			if (LensFile->HasFocusEncoderMapping())
			{
				CineCameraComponent->FocusSettings.ManualFocusDistance = LensFile->EvaluateNormalizedFocus(LensFileEvalData.Input.Focus);
			}
			else if (StaticData->bIsFocusDistanceSupported)
			{
				//If focus is streamed in, query the mapping if there is one. Otherwise, assume focus is usable as is
				CineCameraComponent->FocusSettings.ManualFocusDistance = LensFileEvalData.Input.Focus;
			}

			// Update the minimum focus of the camera (if needed)
			CineCameraComponent->LensSettings.MinimumFocusDistance = FMath::Min(CineCameraComponent->LensSettings.MinimumFocusDistance, CineCameraComponent->FocusSettings.ManualFocusDistance);
		}

		if (UpdateFlags.bApplyAperture)
		{
			//If Iris encoder mapping is present, use it. If not, use incoming value directly if streamed in
			if (LensFile->HasIrisEncoderMapping())
			{
				CineCameraComponent->CurrentAperture = LensFile->EvaluateNormalizedIris(LensFileEvalData.Input.Iris);
			}
			else if (StaticData->bIsApertureSupported)
			{
				CineCameraComponent->CurrentAperture = LensFileEvalData.Input.Iris;
			}

			// Update the minimum and maximum aperture of the camera (if needed)
			CineCameraComponent->LensSettings.MinFStop = FMath::Min(CineCameraComponent->LensSettings.MinFStop, CineCameraComponent->CurrentAperture);
			CineCameraComponent->LensSettings.MaxFStop = FMath::Max(CineCameraComponent->LensSettings.MaxFStop, CineCameraComponent->CurrentAperture);
		}
	}
	else
	{
		//Use Min/Max values of each component to remap normalized incoming values
		if (bUseCameraRange)
		{
			if (StaticData->bIsFocusDistanceSupported && UpdateFlags.bApplyFocusDistance)
			{
				const float MinFocusDistanceInWorldUnits = CineCameraComponent->LensSettings.MinimumFocusDistance * (CineCameraComponent->GetWorldToMetersScale() / 1000.f);	// convert mm to uu
				const float NewFocusDistance = FMath::Lerp(MinFocusDistanceInWorldUnits, 100000.0f, FrameData->FocusDistance);
				CineCameraComponent->FocusSettings.ManualFocusDistance = NewFocusDistance;
			}
			if (StaticData->bIsApertureSupported && UpdateFlags.bApplyAperture)
			{
				const float NewAperture = FMath::Lerp(CineCameraComponent->LensSettings.MinFStop, CineCameraComponent->LensSettings.MaxFStop, FrameData->Aperture);
				CineCameraComponent->CurrentAperture = NewAperture;
			}
		}
		else
		{
			if (StaticData->bIsFocusDistanceSupported && UpdateFlags.bApplyFocusDistance)
			{
				if (FrameData->FocusDistance > 1.0f)
				{
					CineCameraComponent->FocusSettings.ManualFocusDistance = FrameData->FocusDistance;
				}
			}

			if (StaticData->bIsApertureSupported && UpdateFlags.bApplyAperture)
			{
				if (FrameData->Aperture > 1.0f)
				{
					CineCameraComponent->CurrentAperture = FrameData->Aperture;
				}
			}
		}
	}

	// Handle Focal Length
	if (StaticData->bIsFocalLengthSupported && UpdateFlags.bApplyFocalLength)
	{
		if (bUseCameraRange)
		{
			const float NewZoom = FMath::Lerp(CineCameraComponent->LensSettings.MinFocalLength, CineCameraComponent->LensSettings.MaxFocalLength, FrameData->FocalLength);
			CineCameraComponent->SetCurrentFocalLength(NewZoom);
		}
		else if (FrameData->FocalLength > 1.0f)
		{
			CineCameraComponent->SetCurrentFocalLength(FrameData->FocalLength);
		}
	}
}

void ULiveLinkCameraController::ApplyNodalOffset(ULensFile* SelectedLensFile, UCineCameraComponent* CineCameraComponent)
{
	// The use of this function has been deprecated.
}

void ULiveLinkCameraController::ApplyDistortion(ULensFile* LensFile, UCineCameraComponent* CineCameraComponent, const FLiveLinkCameraStaticData* StaticData, const FLiveLinkCameraFrameData* FrameData)
{
	// The use of this function has been deprecated.
}

void ULiveLinkCameraController::OnPostActorTick(UWorld* World, ELevelTick TickType, float DeltaSeconds)
{
	// The use of this callback by this class has been deprecated
}

void ULiveLinkCameraController::VerifyFIZWithLensFileTables(ULensFile* LensFile, const FLiveLinkCameraStaticData* StaticData) const
{
	if (LensFile && StaticData)
	{
		const FLiveLinkSubjectKey FakedSubjectKey = { FGuid(), SelectedSubject.Subject };
		
		if (UpdateFlags.bApplyFocusDistance && StaticData->bIsFocusDistanceSupported == false)
		{
			// If the LiveLink source is not streaming focus, we will default to evaluate at 0.0f.
			// In this case, it is expected that there is only one mapping, so warn the user if this is not the case.
			if (LensFile->EncodersTable.GetNumFocusPoints() > 1)
			{
				static const FName NAME_InvalidFocusMappingWhenNotStreamed = "LiveLinkCamera_FocusNotStreamedInvalidMapping";
				FLiveLinkLog::WarningOnce(NAME_InvalidFocusMappingWhenNotStreamed, FakedSubjectKey, TEXT("Problem applying Focus for subject '%s' using LensFile '%s'. Focus wasn't streamed in and more than one focus mapping was found."), *FakedSubjectKey.SubjectName.ToString(), *LensFile->GetName());
			}
		}

		if (UpdateFlags.bApplyAperture && StaticData->bIsApertureSupported == false)
		{
			// If the LiveLink source is not streaming iris, we will default to evaluate at 0.0f.
			// In this case, it is expected that there is only one mapping, so warn the user if this is not the case.
			if (LensFile->EncodersTable.GetNumIrisPoints() > 1)
			{
				static const FName NAME_InvalidIrisMappingWhenNotStreamed = "LiveLinkCamera_IrisNotStreamedInvalidMapping";
				FLiveLinkLog::WarningOnce(NAME_InvalidIrisMappingWhenNotStreamed, FakedSubjectKey, TEXT("Problem applying Iris for subject '%s' using LensFile '%s'. Iris wasn't streamed in and more than one iris mapping was found."), *FakedSubjectKey.SubjectName.ToString(), *LensFile->GetName());
			}
		}
	}
}

void ULiveLinkCameraController::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	const int32 EnterpriseVersion = GetLinkerCustomVersion(FEnterpriseObjectVersion::GUID);
	if (EnterpriseVersion < FEnterpriseObjectVersion::LiveLinkControllerSplitPerRole)
	{
		AActor* MyActor = GetOuterActor();
		if (MyActor)
		{
			//Make sure all UObjects we use in our post load have been postloaded
			MyActor->ConditionalPostLoad();

			ULiveLinkComponentController* LiveLinkComponent = Cast<ULiveLinkComponentController>(MyActor->GetComponentByClass(ULiveLinkComponentController::StaticClass()));
			if (LiveLinkComponent)
			{
				LiveLinkComponent->ConditionalPostLoad();

				//If the transform controller that was created to drive the TransformRole is the built in one, set its data structure with the one that we had internally
				if (LiveLinkComponent->ControllerMap.Contains(ULiveLinkTransformRole::StaticClass()))
				{
					ULiveLinkTransformController* TransformController = Cast<ULiveLinkTransformController>(LiveLinkComponent->ControllerMap[ULiveLinkTransformRole::StaticClass()]);
					if (TransformController)
					{
						TransformController->ConditionalPostLoad();
						TransformController->TransformData = TransformData_DEPRECATED;
					}
				}

				//if Subjects role direct controller is us, set the component to control to what we had
				if (LiveLinkComponent->SubjectRepresentation.Role == ULiveLinkCameraRole::StaticClass())
				{
					ComponentPicker = ComponentToControl_DEPRECATED;
				}
			}
		}
	}

	const int32 UE5MainVersion = GetLinkerCustomVersion(FUE5MainStreamObjectVersion::GUID);
	if (UE5MainVersion < FUE5MainStreamObjectVersion::LensComponentNodalOffset)
	{
		if (HasAnyFlags(RF_ArchetypeObject))
		{
			return;
		}
		// LensFile and distortion evaluation for nodal offset has moved to the Lens component. In order for existing camera actors to continue functioning as before,
		// we have to migrate some properties from the camera controller to the lens component. If the owning actor does not have already have a lens 
		// component then we will instance a new one.
		AActor* const CameraActor = GetOuterActor();
		if (ensure(CameraActor))
		{
			CameraActor->ConditionalPostLoad();
			if (CameraActor->GetWorld())
			{
				TInlineComponentArray<ULensComponent*> LensComponents;
				CameraActor->GetComponents(LensComponents);

				// Loop through any existing lens components to find those that may already have been evaluating the same LensFile as this camera controller
				bool bFoundMatchingLensComponent = false;
				for (ULensComponent* LensComponent : LensComponents)
				{
					if (LensComponent->GetLensFile() == LensFilePicker.GetLensFile())
					{
						bFoundMatchingLensComponent = true;

						// It is very unlikely that both the camera controller and the existing lens component were both applying nodal offset simultaneously.
						// Therefore, it is safe to assume that if the camera controller was previously applying nodal offset, the new desired behavior would be for 
						// the lens component to start applying it. But if the lens component was previously applying nodal offset, then there is no need to update anything.
						PRAGMA_DISABLE_DEPRECATION_WARNINGS
							if (bApplyNodalOffset_DEPRECATED)
							{
								LensComponent->SetApplyNodalOffsetOnTick(bApplyNodalOffset_DEPRECATED);
							}

						// Copy the overscan multiplier settings to the lens component
						if (bScaleOverscan_DEPRECATED)
						{
							LensComponent->SetOverscanMultiplier(OverscanMultiplier_DEPRECATED);
						}

						// Copy the cropped filmback settings
						if (bUseCroppedFilmback_DEPRECATED)
						{
							LensComponent->SetFilmbackOverrideSetting(EFilmbackOverrideSource::CroppedFilmbackSetting);
							LensComponent->SetCroppedFilmback(CroppedFilmback_DEPRECATED);
						}

						PRAGMA_ENABLE_DEPRECATION_WARNINGS
					}
				}

				// If no matching existing lens components were found, create a new one and initialize it with the correct LensFile
				if (!bFoundMatchingLensComponent)
				{
					if (LensFilePicker.GetLensFile())
					{
						ULensComponent* LensComponent = NewObject<ULensComponent>(CameraActor, MakeUniqueObjectName(this, ULensComponent::StaticClass(), TEXT("Lens")));
						CameraActor->AddInstanceComponent(LensComponent);
						LensComponent->RegisterComponent();

						LensComponent->SetLensFilePicker(LensFilePicker);

						PRAGMA_DISABLE_DEPRECATION_WARNINGS
							LensComponent->SetApplyNodalOffsetOnTick(bApplyNodalOffset_DEPRECATED);

						// Copy the overscan multiplier settings to the lens component
						if (bScaleOverscan_DEPRECATED)
						{
							LensComponent->SetOverscanMultiplier(OverscanMultiplier_DEPRECATED);
						}

						// Copy the cropped filmback settings
						if (bUseCroppedFilmback_DEPRECATED)
						{
							LensComponent->SetFilmbackOverrideSetting(EFilmbackOverrideSource::CroppedFilmbackSetting);
							LensComponent->SetCroppedFilmback(CroppedFilmback_DEPRECATED);
						}
						PRAGMA_ENABLE_DEPRECATION_WARNINGS
					}
				}
			}
		}

		// If this controller was previously applying nodal offset, reset the camera component's transform to its original pose
		if (UCineCameraComponent* const CineCameraComponent = Cast<UCineCameraComponent>(GetAttachedComponent()))
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (bApplyNodalOffset_DEPRECATED)
			{
				CineCameraComponent->SetRelativeLocation(OriginalCameraLocation_DEPRECATED);
				CineCameraComponent->SetRelativeRotation(OriginalCameraRotation_DEPRECATED.Quaternion());
			}
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}
#endif //WITH_EDITOR
}

const FLensFileEvalData& ULiveLinkCameraController::GetLensFileEvalDataRef() const
{
	return LensFileEvalData;
}
