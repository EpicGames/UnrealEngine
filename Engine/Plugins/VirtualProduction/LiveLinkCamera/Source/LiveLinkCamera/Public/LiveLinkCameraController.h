// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkControllerBase.h"

#include "CineCameraComponent.h"
#include "Controllers/LiveLinkTransformController.h"
#include "Engine/EngineTypes.h"
#include "LensDistortionModelHandlerBase.h"
#include "LensFile.h"

#include "LiveLinkCameraController.generated.h"


struct FLiveLinkCameraStaticData;
struct FLiveLinkCameraFrameData;

/** Flags to control whether incoming values from LiveLink Camera FrameData should be applied or not */
USTRUCT()
struct FLiveLinkCameraControllerUpdateFlags
{
	GENERATED_BODY()

	/** Whether to apply FOV if it's available in LiveLink FrameData */
	UPROPERTY(EditAnywhere, Category = "Updates")
	bool bApplyFieldOfView = true;
	
	/** Whether to apply Aspect Ratio if it's available in LiveLink FrameData */
	UPROPERTY(EditAnywhere, Category = "Updates")
	bool bApplyAspectRatio = true;

	/** Whether to apply Focal Length if it's available in LiveLink FrameData */
	UPROPERTY(EditAnywhere, Category = "Updates")
	bool bApplyFocalLength = true;

	/** Whether to apply Projection Mode if it's available in LiveLink FrameData */
	UPROPERTY(EditAnywhere, Category = "Updates")
	bool bApplyProjectionMode= true;

	/** Whether to apply Filmback if it's available in LiveLink StaticData or sent in the LiveLink FrameData*/
	UPROPERTY(EditAnywhere, Category = "Updates")
	bool bApplyFilmBack = true;

	/** Whether to apply Aperture if it's available in LiveLink FrameData */
	UPROPERTY(EditAnywhere, Category = "Updates")
	bool bApplyAperture = true;

	/** Whether to apply Focus Distance if it's available in LiveLink FrameData */
	UPROPERTY(EditAnywhere, Category = "Updates")
	bool bApplyFocusDistance = true;
};

/**
 */
UCLASS()
class LIVELINKCAMERA_API ULiveLinkCameraController : public ULiveLinkControllerBase
{
	GENERATED_BODY()

public:
	ULiveLinkCameraController();

	//~ Begin ULiveLinkControllerBase interface
	virtual void Tick(float DeltaTime, const FLiveLinkSubjectFrameData& SubjectData) override;
	virtual bool IsRoleSupported(const TSubclassOf<ULiveLinkRole>& RoleToSupport) override;
	virtual TSubclassOf<UActorComponent> GetDesiredComponentClass() const override;
	//~ End ULiveLinkControllerBase interface

	//~ Begin UObject interface
	virtual void PostLoad() override;
	//~ End UObject interface

	/** Returns a const reference to input data used to evaluate the lens file */
	const FLensFileEvalData& GetLensFileEvalDataRef() const;

	UE_DEPRECATED(5.1, "This function has been deprecated. Nodal Offset is now applied by the LensComponent")
	void SetApplyNodalOffset(bool bInApplyNodalOffset);

protected:
	/** Applies FIZ data coming from LiveLink stream. Lens file is used if encoder mapping is required  */
	void ApplyFIZ(ULensFile* LensFile, UCineCameraComponent* CineCameraComponent, const FLiveLinkCameraStaticData* StaticData, const FLiveLinkCameraFrameData* FrameData);

	UE_DEPRECATED(5.1, "This function has been deprecated. Nodal Offset is now applied by the LensComponent")
	void ApplyNodalOffset(ULensFile* LensFile, UCineCameraComponent* CineCameraComponent);

	UE_DEPRECATED(5.1, "This function has been deprecated. Distortion is now applied by the LensComponent")
	void ApplyDistortion(ULensFile* LensFile, UCineCameraComponent* CineCameraComponent, const FLiveLinkCameraStaticData* StaticData, const FLiveLinkCameraFrameData* FrameData);

	UE_DEPRECATED(5.1, "The use of this callback by this class has been deprecated and it is no longer registered. You can register your own delegate with FWorldDelegates::OnWorldPostActorTick")
	void OnPostActorTick(UWorld* World, ELevelTick TickType, float DeltaSeconds);

	/** 
	 * If part of FIZ is not streamed, verify that LensFile associated tables have only one entry 
	 * Used to warn user of potential problem evaluating LensFile
	 */
	void VerifyFIZWithLensFileTables(ULensFile* LensFile, const FLiveLinkCameraStaticData* StaticData) const;

public:
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FComponentReference ComponentToControl_DEPRECATED;

	UPROPERTY()
	FLiveLinkTransformControllerData TransformData_DEPRECATED;
#endif

	/**
	 * Should LiveLink inputs be remapped (i.e normalized to physical units) using camera component range
	 */
	UPROPERTY(EditAnywhere, Category = "Camera Calibration")
	bool bUseCameraRange = false;

	/** Asset containing encoder and fiz mapping */
	UPROPERTY(EditAnywhere, Category = "Camera Calibration")
	FLensFilePicker LensFilePicker;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.1, "This property has been deprecated. The LiveLink camera controller no longer applies nodal offset.")
	UPROPERTY()
	bool bApplyNodalOffset_DEPRECATED = true;

	UE_DEPRECATED(5.1, "This property has been deprecated. Use the corresponding setting in the LensComponent instead.")
	UPROPERTY()
	bool bUseCroppedFilmback_DEPRECATED = false;

	UE_DEPRECATED(5.1, "This property has been deprecated. Use the corresponding setting in the LensComponent instead.")
	UPROPERTY()
	FCameraFilmbackSettings CroppedFilmback_DEPRECATED;

	UE_DEPRECATED(5.1, "This property has been deprecated. Use the corresponding setting in the LensComponent instead.")
	UPROPERTY()
	bool bScaleOverscan_DEPRECATED = false;

	UE_DEPRECATED(5.1, "This property has been deprecated. Use the corresponding setting in the LensComponent instead.")
	UPROPERTY()
	float OverscanMultiplier_DEPRECATED = 1.0f;
#endif //WITH_EDITORONLY_DATA

protected:

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.1, "This property has been deprecated. The LiveLink camera controller no longer applies nodal offset.")
	UPROPERTY()
	FRotator OriginalCameraRotation_DEPRECATED;

	UE_DEPRECATED(5.1, "This property has been deprecated. The LiveLink camera controller no longer applies nodal offset.")
	UPROPERTY()
	FVector OriginalCameraLocation_DEPRECATED;

	UE_DEPRECATED(5.1, "This property has been deprecated. The LiveLink camera controller no longer evaluated distortion.")
	UPROPERTY(Transient)
	TObjectPtr<ULensDistortionModelHandlerBase> LensDistortionHandler_DEPRECATED = nullptr;

	UE_DEPRECATED(5.1, "This property has been deprecated. The LiveLink camera controller no longer evaluated distortion.")
	UPROPERTY(DuplicateTransient)
	FGuid DistortionProducerID_DEPRECATED;
#endif //WITH_EDITORONLY_DATA

	/** Used to control which data from LiveLink is actually applied to camera */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FLiveLinkCameraControllerUpdateFlags UpdateFlags;

#if WITH_EDITORONLY_DATA
	/** Whether to refresh frustum drawing on value change */
	UPROPERTY(EditAnywhere, Category = "Debug")
	bool bShouldUpdateVisualComponentOnChange = true;
#endif

protected:
	/** Caches the latest inputs to the LensFile evaluation */
	FLensFileEvalData LensFileEvalData;

private:
	double LastLensTableVerificationTimestamp = 0.0;

	bool bLastIsDepthOfFieldSupported = false;
};
