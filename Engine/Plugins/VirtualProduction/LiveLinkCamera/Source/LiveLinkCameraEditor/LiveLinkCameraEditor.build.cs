// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class LiveLinkCameraEditor : ModuleRules
{
	public LiveLinkCameraEditor(ReadOnlyTargetRules Target) : base(Target)
	{		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CameraCalibrationCore",
				"Core",
				"CoreUObject",
				"DetailCustomizations",
				
				"LiveLinkInterface",
				"LiveLinkCamera",
				"LiveLinkComponents",
				"PropertyEditor",
				"Slate",
				"SlateCore"
			}
		);
	}
}
