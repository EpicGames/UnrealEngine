// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkCameraControllerCustomization.h"



#define LOCTEXT_NAMESPACE "FFileMediaSourceCustomization"


/* IDetailCustomization interface
 *****************************************************************************/

void FLiveLinkCameraControllerCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	//Disabling customization for now. Might add one for the lens picker to show the default lens file asset ini the 
	//lens file picker instead and do customization at that level instead of camera controller

#if 0
	TArray<TWeakObjectPtr<UObject>> SelectedObjects = DetailBuilder.GetSelectedObjects();

	
	//If more than one don't add up warning icon logic
	if (SelectedObjects.Num() != 1)
	{
		return;
	}

	if (ULiveLinkComponentController* SelectedPtr = Cast<ULiveLinkComponentController>(SelectedObjects[0].Get()))
	{
		EditedObject = SelectedPtr;

		IDetailCategoryBuilder& LensCategory = DetailBuilder.EditCategory("Lens");
		{
			//Customize LensFile property to show a warning if it's needed

			TSharedPtr<IPropertyHandle> LensFileProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULiveLinkCameraController, LensFilePicker));
			{
				IDetailPropertyRow& LensFileRow = LensCategory.AddProperty(LensFileProperty);
				LensFileRow
					.ShowPropertyButtons(false)
					.CustomWidget()
					.NameContent()
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						LensFileProperty->CreatePropertyNameWidget()
					]
				+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.Padding(4.0f, 0.0f, 0.0f, 0.0f)
					[
						SNew(SImage)
						.Image(FCoreStyle::Get().GetBrush("Icons.Warning"))
					.ToolTipText(LOCTEXT("LensFileWarning", "The selected LiveLink subject requires encoder mapping. The current lens file is invalid."))
					.Visibility(this, &FLiveLinkCameraControllerCustomization::HandleEncoderMappingWarningIconVisibility)
					]
					]
				.ValueContent()
					.MinDesiredWidth(250.f)
					[
						LensFileProperty->CreatePropertyValueWidget()
					];
				}
		}
	}
#endif

}


#undef LOCTEXT_NAMESPACE
