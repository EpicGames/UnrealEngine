// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

#include "UObject/WeakObjectPtr.h"

struct EVisibility;

class IDetailLayoutBuilder;
class IPropertyHandle;
class ULiveLinkComponentController;

/**
 * Implements a details view customization for the ULiveLinkCameraController class.
 */
class FLiveLinkCameraControllerCustomization : public IDetailCustomization
{
public:

	//~ Begin IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	//~ End IDetailCustomization interface

public:

	/**
	 * Creates an instance of this class.
	 *
	 * @return The new instance.
	 */
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FLiveLinkCameraControllerCustomization>();
	}

private:

	/** Callback for getting the selected path in the URL picker widget. */
	FString HandleFilePathPickerFilePath() const;

	/** Callback for getting the file type filter for the URL picker. */
	FString HandleFilePathPickerFileTypeFilter() const;

	/** Callback for picking a path in the URL picker. */
	void HandleFilePathPickerPathPicked(const FString& PickedPath);

	/** Callback for getting the visibility of warning icon for invalid encoder mapping */
	EVisibility HandleEncoderMappingWarningIconVisibility() const;

private:

	/** LiveLinkComponent on which we're acting */
	TWeakObjectPtr<ULiveLinkComponentController> EditedObject;
};
