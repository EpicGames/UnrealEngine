// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkCameraEditorModule.h"

#include "LiveLinkCameraController.h"
#include "LiveLinkCameraControllerCustomization.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"

LLM_DEFINE_TAG(LiveLink_LiveLinkCameraEditor);

void FLiveLinkCameraEditorModule::StartupModule()
{
	RegisterCustomizations();
}

void FLiveLinkCameraEditorModule::ShutdownModule()
{
	UnregisterCustomizations();
}

void FLiveLinkCameraEditorModule::RegisterCustomizations()
{
	LLM_SCOPE_BYTAG(LiveLink_LiveLinkCameraEditor);

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyEditorModule.RegisterCustomClassLayout(ULiveLinkCameraController::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FLiveLinkCameraControllerCustomization::MakeInstance));
}

void FLiveLinkCameraEditorModule::UnregisterCustomizations()
{
	LLM_SCOPE_BYTAG(LiveLink_LiveLinkCameraEditor);

	if (UObjectInitialized() && !IsEngineExitRequested())
	{
		FPropertyEditorModule* PropertyEditorModule = FModuleManager::Get().GetModulePtr<FPropertyEditorModule>("PropertyEditor");
		if (PropertyEditorModule)
		{
			PropertyEditorModule->UnregisterCustomClassLayout(ULiveLinkCameraController::StaticClass()->GetFName());
		}
	}
}

IMPLEMENT_MODULE(FLiveLinkCameraEditorModule, LiveLinkCameraEditor)


