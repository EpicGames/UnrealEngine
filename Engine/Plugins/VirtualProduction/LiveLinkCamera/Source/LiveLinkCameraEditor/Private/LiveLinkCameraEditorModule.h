// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"


class FLiveLinkCameraEditorModule : public IModuleInterface
{
public:

	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface

private:
	void RegisterCustomizations();
	void UnregisterCustomizations();
};
