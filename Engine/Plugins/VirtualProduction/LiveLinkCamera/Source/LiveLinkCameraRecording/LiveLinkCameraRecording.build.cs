// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class LiveLinkCameraRecording : ModuleRules
{
	public LiveLinkCameraRecording(ReadOnlyTargetRules Target) : base(Target)
	{		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CameraCalibrationCore",
				"CinematicCamera",
				"Core",
				"CoreUObject",
				"Engine",
				"LevelSequence",
				"LiveLinkCamera",
				"LiveLinkComponents",
				"LiveLinkInterface",
				"LiveLinkMovieScene",
				"LiveLinkSequencer",
				"MovieScene",
				"MovieSceneTools",
				"MovieSceneTracks",
				"Sequencer",
				"Slate",
				"SlateCore",
				"UnrealEd",
				"TakeTrackRecorders",
			}
		);
	}
}
