// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkCameraControllerTrackEditor.h"

#include "LevelSequence.h"

#define LOCTEXT_NAMESPACE "LiveLinkCameraControllerTrackEditor"

TSharedRef<ISequencerTrackEditor> FLiveLinkCameraControllerTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> InSequencer)
{
	return MakeShared<FLiveLinkCameraControllerTrackEditor>(InSequencer);
}

FText FLiveLinkCameraControllerTrackEditor::GetDisplayName() const
{
	return LOCTEXT("LiveLinkCameraControllerTrackEditor_DisplayName", "Live Link Camera Controller");
}

bool FLiveLinkCameraControllerTrackEditor::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	ETrackSupport TrackSupported = InSequence ? InSequence->IsTrackSupported(UMovieSceneLiveLinkCameraControllerTrack::StaticClass()) : ETrackSupport::Default;

	if (TrackSupported == ETrackSupport::NotSupported)
	{
		return false;
	}

	return (InSequence && InSequence->IsA(ULevelSequence::StaticClass())) || TrackSupported == ETrackSupport::Supported;
}

bool FLiveLinkCameraControllerTrackEditor::SupportsType(TSubclassOf<UMovieSceneTrack> Type) const
{
	return (Type == UMovieSceneLiveLinkCameraControllerTrack::StaticClass());
}

#undef LOCTEXT_NAMESPACE
