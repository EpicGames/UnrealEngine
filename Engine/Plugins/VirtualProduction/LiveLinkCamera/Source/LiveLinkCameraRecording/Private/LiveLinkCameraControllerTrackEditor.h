// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "KeyframeTrackEditor.h"
#include "MovieSceneLiveLinkCameraControllerTrack.h"

/** A property track editor for Live Link Camera Controller properties. */
class FLiveLinkCameraControllerTrackEditor : public FKeyframeTrackEditor<UMovieSceneLiveLinkCameraControllerTrack>
{
public:

	FLiveLinkCameraControllerTrackEditor(TSharedRef<ISequencer> InSequencer)
		: FKeyframeTrackEditor<UMovieSceneLiveLinkCameraControllerTrack>(InSequencer)
	{
	}

	/**
	* Creates an instance of this class.  Called by a sequencer
	*
	* @param OwningSequencer The sequencer instance to be used by this tool
	* @return The new instance of this class
	*/
	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer);

public:

	//~ Begin ISequencerTrackEditor interface
	virtual FText GetDisplayName() const override;
	virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneTrack> Type) const override;
	//~ End ISequencerTrackEditor interface
};