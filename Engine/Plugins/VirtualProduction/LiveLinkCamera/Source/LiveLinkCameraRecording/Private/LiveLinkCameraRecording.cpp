// Copyright Epic Games, Inc. All Rights Reserved.

#include "ISequencerModule.h"
#include "LiveLinkCameraControllerTrackEditor.h"
#include "Modules/ModuleManager.h"

LLM_DEFINE_TAG(LiveLinkCamera_LiveLinkCameraRecording);

class FLiveLinkCameraRecordingModule : public IModuleInterface
{
public:
	// IModuleInterface interface
	virtual void StartupModule() override
	{
		LLM_SCOPE_BYTAG(LiveLinkCamera_LiveLinkCameraRecording);

		ISequencerModule& SequencerModule = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer");
		CreateLiveLinkCameraControllerTrackEditorHandle = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic(&FLiveLinkCameraControllerTrackEditor::CreateTrackEditor));
	}

	virtual void ShutdownModule() override
	{
		LLM_SCOPE_BYTAG(LiveLinkCamera_LiveLinkCameraRecording);

		ISequencerModule* SequencerModule = FModuleManager::GetModulePtr<ISequencerModule>("Sequencer");
		if (SequencerModule != nullptr)
		{
			SequencerModule->UnRegisterTrackEditor(CreateLiveLinkCameraControllerTrackEditorHandle);
		}
	}

private:

	FDelegateHandle CreateLiveLinkCameraControllerTrackEditorHandle;
};

IMPLEMENT_MODULE(FLiveLinkCameraRecordingModule, LiveLinkCameraRecording);