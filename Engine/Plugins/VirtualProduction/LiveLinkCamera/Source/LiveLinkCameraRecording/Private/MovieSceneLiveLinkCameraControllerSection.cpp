// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneLiveLinkCameraControllerSection.h"

#include "CineCameraComponent.h"
#include "EntitySystem/MovieSceneSharedPlaybackState.h"
#include "Evaluation/MovieSceneEvaluationState.h"
#include "LiveLinkCameraController.h"
#include "LiveLinkComponentController.h"
#include "Roles/LiveLinkCameraRole.h"

void UMovieSceneLiveLinkCameraControllerSection::Initialize(ULiveLinkControllerBase* InLiveLinkController)
{
}

void UMovieSceneLiveLinkCameraControllerSection::Update(TSharedRef<FSharedPlaybackState> SharedPlaybackState, const UE::MovieScene::FEvaluationHookParams& Params) const
{
	if (!CachedLensFile || !bApplyNodalOffsetFromCachedLensFile)
	{
		return;
	}

	TArrayView<TWeakObjectPtr<>> BoundObjects = SharedPlaybackState->FindBoundObjects(Params.ObjectBindingID, Params.SequenceID);
	for (TWeakObjectPtr<>& BoundObject : BoundObjects)
	{
		if (ULiveLinkComponentController* LiveLinkComponent = Cast<ULiveLinkComponentController>(BoundObject.Get()))
		{
			// Find the LL camera controller in the component's controller map			
			if (TObjectPtr<ULiveLinkControllerBase>* Controller = LiveLinkComponent->ControllerMap.Find(ULiveLinkCameraRole::StaticClass()))
			{
				if (ULiveLinkCameraController* CameraController = Cast<ULiveLinkCameraController>(*Controller))
				{
					UActorComponent* CurrentComponentToControl = CameraController->GetAttachedComponent();
 					if (UCineCameraComponent* CineCameraComponent = Cast<UCineCameraComponent>(CurrentComponentToControl))
 					{
						const FLensFileEvalData& LensFileEvalData = CameraController->GetLensFileEvalDataRef();

						FNodalPointOffset Offset;
						CachedLensFile->EvaluateNodalPointOffset(LensFileEvalData.Input.Focus, LensFileEvalData.Input.Zoom, Offset);

						CineCameraComponent->AddLocalOffset(Offset.LocationOffset);
						CineCameraComponent->AddLocalRotation(Offset.RotationOffset);
 					}
				}
			}
		}
	}
}
