// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sections/MovieSceneHookSection.h"


#include "MovieSceneLiveLinkCameraControllerSection.generated.h"

class ULensFile;
namespace UE::MovieScene { struct FEvaluationHookParams; }

class ULiveLinkControllerBase;

/** Movie Scene section for LiveLink Camera Controller properties */
UCLASS()
class LIVELINKCAMERARECORDING_API UMovieSceneLiveLinkCameraControllerSection : public UMovieSceneHookSection
{
	GENERATED_BODY()

public:

	UMovieSceneLiveLinkCameraControllerSection(const FObjectInitializer& ObjectInitializer)
		: UMovieSceneHookSection(ObjectInitializer) 
	{ 
		bRequiresRangedHook = true; 
	}

	/** Initialize the movie scene section with the LiveLink controller whose properties it will record */
	void Initialize(ULiveLinkControllerBase* InLiveLinkController);

	//~ Begin UMovieSceneHookSection interface
	virtual void Update(TSharedRef<FSharedPlaybackState> SharedPlaybackState, const UE::MovieScene::FEvaluationHookParams& Params) const override;
	//~ End UMovieSceneHookSection interface

public:
	/** If true, the Cached Lens File will be evaluated for nodal offset, and that offset will be applied to the transform of the CameraComponent */
	UPROPERTY(EditAnywhere, Category="Camera Calibration")
	bool bApplyNodalOffsetFromCachedLensFile = true;

private:
	/** Saved duplicate of the LensFile asset used by the recorded LiveLink Camera Controller at the time of recording */
	UPROPERTY(VisibleAnywhere, Category="Camera Calibration")
	TObjectPtr<ULensFile> CachedLensFile = nullptr;
};
