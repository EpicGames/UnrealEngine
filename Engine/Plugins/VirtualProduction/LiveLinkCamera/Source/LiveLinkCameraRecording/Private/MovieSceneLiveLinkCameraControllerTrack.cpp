// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneLiveLinkCameraControllerTrack.h"

#include "MovieSceneLiveLinkCameraControllerSection.h"

#define LOCTEXT_NAMESPACE "MovieSceneLiveLinkCameraControllerTrack"


bool UMovieSceneLiveLinkCameraControllerTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneLiveLinkCameraControllerSection::StaticClass();
}

UMovieSceneSection* UMovieSceneLiveLinkCameraControllerTrack::CreateNewSection()
{
	return NewObject<UMovieSceneSection>(this, UMovieSceneLiveLinkCameraControllerSection::StaticClass(), NAME_None, RF_Transactional);
}

#if WITH_EDITORONLY_DATA
FText UMovieSceneLiveLinkCameraControllerTrack::GetDisplayName() const
{
	return LOCTEXT("LiveLinkCameraControllerTrackDisplayName", "LiveLinkCameraController");
}
#endif

void UMovieSceneLiveLinkCameraControllerTrack::AddSection(UMovieSceneSection& Section)
{
	Sections.Add(&Section);
}

const TArray<UMovieSceneSection*>& UMovieSceneLiveLinkCameraControllerTrack::GetAllSections() const
{
	return Sections;
}

bool UMovieSceneLiveLinkCameraControllerTrack::HasSection(const UMovieSceneSection& Section) const
{
	return Sections.Contains(&Section);
}

bool UMovieSceneLiveLinkCameraControllerTrack::IsEmpty() const
{
	return Sections.Num() != 0;
}

void UMovieSceneLiveLinkCameraControllerTrack::RemoveAllAnimationData()
{
	Sections.Empty();
}

void UMovieSceneLiveLinkCameraControllerTrack::RemoveSection(UMovieSceneSection& Section)
{
	Sections.Remove(&Section);
}

void UMovieSceneLiveLinkCameraControllerTrack::RemoveSectionAt(int32 SectionIndex)
{
	Sections.RemoveAt(SectionIndex);
}


#undef LOCTEXT_NAMESPACE
