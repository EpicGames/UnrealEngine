// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "MovieSceneNameableTrack.h"
#include "MovieSceneSection.h"
#include "MovieSceneLiveLinkCameraControllerTrack.generated.h"

/** Movie Scene track for LiveLink Camera Controller properties */
UCLASS()
class LIVELINKCAMERARECORDING_API UMovieSceneLiveLinkCameraControllerTrack : public UMovieSceneNameableTrack
{
	GENERATED_BODY()

public:
	//~ Begin UMovieSceneTrack interface
	virtual void AddSection(UMovieSceneSection& Section) override;
	virtual const TArray<UMovieSceneSection*>& GetAllSections() const override;
	virtual bool HasSection(const UMovieSceneSection& Section) const override;
	virtual bool IsEmpty() const override;
	virtual void RemoveAllAnimationData() override;
	virtual void RemoveSection(UMovieSceneSection& Section) override;
	virtual void RemoveSectionAt(int32 SectionIndex) override;

	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual UMovieSceneSection* CreateNewSection() override;

#if WITH_EDITORONLY_DATA
	virtual FText GetDisplayName() const override;
#endif
	//~ End UMovieSceneTrack interface

protected:
	/** Array of movie scene sections managed by this track */
	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSection>> Sections;
};
