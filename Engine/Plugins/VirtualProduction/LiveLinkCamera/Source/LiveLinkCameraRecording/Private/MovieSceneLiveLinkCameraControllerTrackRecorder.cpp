// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneLiveLinkCameraControllerTrackRecorder.h"

#include "LiveLinkCameraController.h"
#include "MovieSceneLiveLinkCameraControllerTrack.h"

static const FName ControllerPropertyName = TEXT("LiveLinkCameraController");

bool UMovieSceneLiveLinkCameraControllerTrackRecorder::IsLiveLinkControllerSupported(const TSubclassOf<ULiveLinkControllerBase>& ControllerToSupport) const
{
	return ControllerToSupport == ULiveLinkCameraController::StaticClass();
}

void UMovieSceneLiveLinkCameraControllerTrackRecorder::CreateTrackImpl()
{
	UMovieSceneLiveLinkCameraControllerTrack* LiveLinkControllerTrack = MovieScene->FindTrack<UMovieSceneLiveLinkCameraControllerTrack>(ObjectGuid);
	if (!LiveLinkControllerTrack)
	{
		LiveLinkControllerTrack = MovieScene->AddTrack<UMovieSceneLiveLinkCameraControllerTrack>(ObjectGuid);
	}
	else
	{
		LiveLinkControllerTrack->RemoveAllAnimationData();
	}

	if (LiveLinkControllerTrack)
	{
		MovieSceneSection = Cast<UMovieSceneLiveLinkCameraControllerSection>(LiveLinkControllerTrack->CreateNewSection());

		if (MovieSceneSection.IsValid())
		{
			LiveLinkControllerTrack->AddSection(*MovieSceneSection);

			MovieSceneSection->Initialize(LiveLinkControllerToRecord);
		}
	}
}

void UMovieSceneLiveLinkCameraControllerTrackRecorder::RecordSampleImpl(const FQualifiedFrameTime& CurrentTime)
{
	if (MovieSceneSection.IsValid())
	{
		FFrameRate TickResolution = MovieSceneSection->GetTypedOuter<UMovieScene>()->GetTickResolution();
		FFrameNumber CurrentFrame = CurrentTime.ConvertTo(TickResolution).FloorToFrame();
		MovieSceneSection->SetEndFrame(CurrentFrame);
	}
}
