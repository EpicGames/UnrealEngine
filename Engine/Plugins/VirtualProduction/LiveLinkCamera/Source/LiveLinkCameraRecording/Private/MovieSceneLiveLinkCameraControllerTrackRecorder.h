// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneLiveLinkControllerTrackRecorder.h"

#include "MovieSceneLiveLinkCameraControllerSection.h"

#include "MovieSceneLiveLinkCameraControllerTrackRecorder.generated.h"

/** Movie Scene track recorder for LiveLink Camera Controller properties */
UCLASS(BlueprintType)
class LIVELINKCAMERARECORDING_API UMovieSceneLiveLinkCameraControllerTrackRecorder : public UMovieSceneLiveLinkControllerTrackRecorder
{
	GENERATED_BODY()

public:
	//~ Begin UMovieSceneTrackRecorder Interface
	virtual void CreateTrackImpl() override;
	virtual void RecordSampleImpl(const FQualifiedFrameTime& CurrentTime) override;
	virtual UMovieSceneSection* GetMovieSceneSection() const override { return MovieSceneSection.Get(); }
	//~ End UMovieSceneTrackRecorder Interface

	//~ Begin UMovieSceneLiveLinkControllerTrackRecorder Interface
	virtual bool IsLiveLinkControllerSupported(const TSubclassOf<ULiveLinkControllerBase>& ControllerToSupport) const override;
	//~ End UMovieSceneLiveLinkControllerTrackRecorder Interface

private:
	/** Section to record to on the track */
	TWeakObjectPtr<UMovieSceneLiveLinkCameraControllerSection> MovieSceneSection;
};
